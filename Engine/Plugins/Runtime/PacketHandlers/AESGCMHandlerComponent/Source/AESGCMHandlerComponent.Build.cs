// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AESGCMHandlerComponent : ModuleRules
{
	public AESGCMHandlerComponent(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"PacketHandler",
				"PlatformCrypto",
				"PlatformCryptoContext",
				"NetCore"
			}
		);
	}
}