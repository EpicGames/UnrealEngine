// Copyright Epic Games, Inc. All Rights Reserved.


// Includes
#include "AESGCMFaultHandler.h"
#include "Net/Core/Connection/NetConnectionFaultRecoveryBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AESGCMFaultHandler)


/**
 * EAESGCMNetResult
 */

#ifndef CASE_ENUM_TO_TEXT_RET
#define CASE_ENUM_TO_TEXT_RET(txt) case txt: ReturnVal = TEXT(#txt); break;
#endif

const TCHAR* LexToString(EAESGCMNetResult Enum)
{
	const TCHAR* ReturnVal = TEXT("::Invalid");

	switch (Enum)
	{
		FOREACH_ENUM_EAESGCMNETRESULT(CASE_ENUM_TO_TEXT_RET)
	}

	while (*ReturnVal != ':')
	{
		ReturnVal++;
	}

	ReturnVal += 2;

	return ReturnVal;
}


/**
 * FAESGCMFaultHandler
 */

void FAESGCMFaultHandler::InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery)
{
	using namespace UE::Net;

	if (FaultRecovery == nullptr)
	{
		FaultRecovery = InFaultRecovery;

		if (FaultRecovery != nullptr && CounterIndex == INDEX_NONE)
		{
			FaultRecovery->GetFaultManager().AddResultHandlerPtr(this);

			CounterIndex = FaultRecovery->AddNewCounter();

			FaultRecovery->RegisterCounterCategory(ENetFaultCounterCategory::NetworkCorruption, CounterIndex);
		}
	}
}

UE::Net::EHandleNetResult FAESGCMFaultHandler::HandleNetResult(UE::Net::FNetResult&& InResult)
{
	using namespace UE::Net;
	using FNetFaultEscalationHandler = FNetConnectionFaultRecoveryBase::FNetFaultEscalationHandler;

	EHandleNetResult ReturnVal = EHandleNetResult::Handled;
	TNetResult<EAESGCMNetResult>* CastedResult = Cast<EAESGCMNetResult>(&InResult);
	EAESGCMNetResult AESGCMResult = (CastedResult != nullptr ? CastedResult->GetResult() : EAESGCMNetResult::Unknown);
	FEscalationCounter CounterIncrement;

	switch (AESGCMResult)
	{
		case EAESGCMNetResult::AESMissingIV:
		case EAESGCMNetResult::AESMissingAuthTag:
		case EAESGCMNetResult::AESMissingPayload:
		case EAESGCMNetResult::AESDecryptionFailed:
		case EAESGCMNetResult::AESZeroLastByte:
		{
			CounterIncrement.Counter++;

			break;
		}

		default:
		{
			ReturnVal = EHandleNetResult::NotHandled;

			break;
		}
	}


	if (ReturnVal != EHandleNetResult::NotHandled && CounterIndex != INDEX_NONE)
	{
		FEscalationCounter& FrameCounter = FaultRecovery->GetFrameCounter(CounterIndex);

		FrameCounter.AccumulateCounter(CounterIncrement);

		ReturnVal = FaultRecovery->NotifyHandledFault(MoveTemp(InResult));
	}
		
	return ReturnVal;
}

