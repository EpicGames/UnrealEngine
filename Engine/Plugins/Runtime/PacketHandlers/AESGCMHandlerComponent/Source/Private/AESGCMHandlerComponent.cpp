// Copyright Epic Games, Inc. All Rights Reserved.

#include "AESGCMHandlerComponent.h"
#include "HAL/ConsoleManager.h"

#include "PlatformCryptoTypes.h"

IMPLEMENT_MODULE( FAESGCMHandlerComponentModule, AESGCMHandlerComponent )

const int32 FAESGCMHandlerComponent::KeySizeInBytes;
const int32 FAESGCMHandlerComponent::BlockSizeInBytes;
const int32 FAESGCMHandlerComponent::IVSizeInBytes;
const int32 FAESGCMHandlerComponent::AuthTagSizeInBytes;

TSharedPtr<HandlerComponent> FAESGCMHandlerComponentModule::CreateComponentInstance(FString& Options)
{
	return MakeShared<FAESGCMHandlerComponent>();
}


FAESGCMHandlerComponent::FAESGCMHandlerComponent()
	: FEncryptionComponent(FName(TEXT("AESGCMHandlerComponent")))
	, bEncryptionEnabled(false)
{
	EncryptionContext = IPlatformCrypto::Get().CreateContext();
}

void FAESGCMHandlerComponent::SetEncryptionData(const FEncryptionData& EncryptionData)
{
	Decryptor.Reset(nullptr);
	Encryptor.Reset(nullptr);

	if (EncryptionData.Key.Num() != KeySizeInBytes)
	{
		UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::SetEncryptionData. NewKey is not %d bytes long, ignoring.", KeySizeInBytes);
		return;
	}

	// Generate random bytes used for encryption packets, make sure first IV byte is non-zero value
	do
	{
		EPlatformCryptoResult RandResult = EncryptionContext->CreateRandomBytes(OutIV);
		if (RandResult == EPlatformCryptoResult::Failure)
		{
			UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::SetEncryptionData: failed to generate IV.");
			return;
		}
	}
	while (OutIV[0] == 0);
	
	// Dummy IV and AuthTag values, Decrytor/Encryptor will be reset with actual values before each use
	uint8 DummyIV[IVSizeInBytes] = { 0 };
	uint8 DummyAuth[AuthTagSizeInBytes] = { 0 };
	Decryptor = EncryptionContext->CreateDecryptor_AES_256_GCM(EncryptionData.Key, DummyIV, DummyAuth);
	Encryptor = EncryptionContext->CreateEncryptor_AES_256_GCM(EncryptionData.Key, DummyIV);
}

void FAESGCMHandlerComponent::EnableEncryption()
{
	bEncryptionEnabled = true;
}

void FAESGCMHandlerComponent::DisableEncryption()
{
	bEncryptionEnabled = false;
}

bool FAESGCMHandlerComponent::IsEncryptionEnabled() const
{
	return bEncryptionEnabled;
}

void FAESGCMHandlerComponent::Initialize()
{
	SetActive(true);
	SetState(UE::Handler::Component::State::Initialized);
	Initialized();
}

void FAESGCMHandlerComponent::InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery)
{
	AESGCMFaultHandler.InitFaultRecovery(InFaultRecovery);
}

bool FAESGCMHandlerComponent::IsValid() const
{
	return true;
}


// Incoming & Outgoing packets will always operate on byte level. No bit-aligned stuff is supported here for efficiency reasons.

// Encrypted packed layout: 
// [iv:12] [auth:16] [ciphertext:N]
// where low byte of first IV byte is non-zero value

// Unencrypted packed layout:
// [0] [plaintext:N]
// where first byte is 0 to signal unencrypted packet

// Any further packet handler is expected to encode their bit-length explicitly if they operate on non-byte aligned packet sizes
// In typical use case there will be OodleNetworkHandler that is byte aligned and it encodes uncompressed length

void FAESGCMHandlerComponent::Incoming(FIncomingPacketRef PacketRef)
{
	using namespace UE::Net;

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler AESGCM Decrypt"), STAT_PacketHandler_AESGCM_Decrypt, STATGROUP_Net);

	FBitReader& Packet = PacketRef.Packet;
	FInPacketTraits& Traits = PacketRef.Traits;

	int64 PacketBytes = Packet.GetNumBytes();

	// Handle this packet
	if (IsValid() && PacketBytes > 0)
	{
		uint8* PacketData = Packet.GetData();

		// If first byte is nonzero, then payload is encrypted
		if (PacketData[0] != 0)
		{
			// If the key hasn't been set yet, we can't decrypt, so ignore this packet. We don't set an error in this case because it may just be an out-of-order packet.
			if (!Decryptor.IsValid())
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Incoming: received encrypted packet before key was set, ignoring.");
				Packet.SetData(nullptr, 0);
				return;
			}

			// First 12 bytes is IV
			TArrayView<uint8> IV { PacketData, IVSizeInBytes };
			if (PacketBytes >= IV.Num())
			{
				PacketData += IV.Num();
				PacketBytes -= IV.Num();
			}
			else
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Incoming: missing IV");

				Packet.SetError();
				AddToChainResultPtr(Traits.ExtendedError, EAESGCMNetResult::AESMissingIV);

				return;
			}

			// Then there are 16 bytes of AuthTag
			TArrayView<uint8> AuthTag { PacketData, AuthTagSizeInBytes };
			if (PacketBytes >= AuthTag.Num())
			{
				PacketData += AuthTag.Num();
				PacketBytes -= AuthTag.Num();
			}
			else
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Incoming: missing auth tag");

				Packet.SetError();
				AddToChainResultPtr(Traits.ExtendedError, EAESGCMNetResult::AESMissingAuthTag);

				return;
			}

			if (PacketBytes == 0)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Incoming: missing ciphertext");

				Packet.SetError();
				AddToChainResultPtr(Traits.ExtendedError, EAESGCMNetResult::AESMissingPayload);

				return;
			}

			// Rest of bytes is ciphertext
			TArrayView<uint8> CipherText { PacketData, static_cast<int32>(PacketBytes) };

			UE_LOGF(PacketHandlerLog, VeryVerbose, "AESGCM packet handler received %d bytes before decryption.", CipherText.Num());
			uint8 PlainText[MAX_PACKET_SIZE];

			// Decrypt payload and verify AuthTag
			EPlatformCryptoResult DecryptResult = Decrypt(PlainText, CipherText, IV, AuthTag);
			if (DecryptResult == EPlatformCryptoResult::Failure)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Incoming: failed to decrypt packet.");

				Packet.SetError();
				AddToChainResultPtr(Traits.ExtendedError, EAESGCMNetResult::AESDecryptionFailed);

				return;
			}

			Packet.SetData(PlainText, CipherText.Num() * 8);
		}
		else
		{
			// Skip first zero byte, rest of bytes contains unencrypted data
			Packet.Skip(8);
		}
	}
}

void FAESGCMHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler AESGCM Encrypt"), STAT_PacketHandler_AESGCM_Encrypt, STATGROUP_Net);

	// Handle this packet
	if (IsValid() && Packet.GetNumBytes() > 0)
	{
		if (bEncryptionEnabled)
		{
			UE_LOGF(PacketHandlerLog, VeryVerbose, "AESGCM packet handler sending %lld bits before encryption.", Packet.GetNumBits());

			TStaticArray<uint8, AuthTagSizeInBytes> AuthTag;
			TArrayView<uint8> Payload { Packet.GetData(), static_cast<int32>(Packet.GetNumBytes()) };

			// Prepare new IV for encryption
			{
				// Use 64-bit counter in bytes [1,9], leave byte [0] unchanged as that is non-zero indication that packet is encrypted
				uint8* CounterLocation = OutIV.GetData() + 1;

				// This place does not need completely new random value every time, just a unique value for each packet.
				// Incrementing IV bytes as 64-bit integer with wrap-around is enough for this use case.
				uint64 Counter = INTEL_ORDER64(FPlatformMemory::ReadUnaligned<uint64>(CounterLocation));
				FPlatformMemory::WriteUnaligned(CounterLocation, INTEL_ORDER64(Counter + 1));
			}

			TArrayView<uint8> PlainText = { Packet.GetData(), static_cast<int32>(Packet.GetNumBytes()) };
			uint8 CipherText[MAX_PACKET_SIZE];

			// Encrypt payload & write AuthTag
			EPlatformCryptoResult EncryptResult = Encrypt(CipherText, PlainText, OutIV, AuthTag);
			if (EncryptResult == EPlatformCryptoResult::Failure)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESGCMHandlerComponent::Outgoing: failed to encrypt packet.");
				Packet.SetError();
				return;
			}

			// Make sure there is enough space allocated for outgoing packet memory
			int64 NewPacketByteCount = IVSizeInBytes + AuthTagSizeInBytes + PlainText.Num();
			if (NewPacketByteCount * 8 > Packet.GetMaxBits())
			{
				// Allocate max MAX_PACKET_SIZE bytes, just like PacketHandler does, so packet memory can be reused later
				check(NewPacketByteCount <= MAX_PACKET_SIZE);
				Packet = FBitWriter(MAX_PACKET_SIZE * 8);
			}

			uint8* NewPacketData = Packet.GetData();

			// Copy the IV, AuthTag and encrypted payload to new packet
			FPlatformMemory::Memcpy(NewPacketData, OutIV.GetData(), OutIV.Num());
			NewPacketData += OutIV.Num();

			FPlatformMemory::Memcpy(NewPacketData, AuthTag.GetData(), AuthTag.Num());
			NewPacketData += AuthTag.Num();

			FPlatformMemory::Memcpy(NewPacketData, CipherText, PlainText.Num());

			// Set how many valid bits there are in the new packet
			Packet.SetNumBits(NewPacketByteCount * 8);

			UE_LOG(PacketHandlerLog, VeryVerbose, TEXT("  AESGCM packet handler sending %" INT64_FMT " bytes after encryption."), Packet.GetNumBytes());
		}
		else
		{
			// Make sure packet has space available for extra 8 bits
			if (Packet.AllowAppend(8))
			{
				// Reserve one byte in packet data in the beginning
				uint8* PacketData = Packet.GetData();
				FPlatformMemory::Memmove(PacketData + 1, PacketData, Packet.GetNumBytes());

				// First byte 0 means that packet contains unencrypted payload
				PacketData[0] = 0;

				// Include first 8 bits in new packet data
				Packet.SetNumBits(8 + Packet.GetNumBits());
			}
			else
			{
				Packet.SetOverflowed(Packet.GetNumBits());
			}
		}
	}
}

EPlatformCryptoResult FAESGCMHandlerComponent::Decrypt(TArrayView<uint8> OutPlaintext, const TArrayView<const uint8> InCiphertext, const TArrayView<const uint8> IV, const TArrayView<const uint8> AuthTag)
{
	EPlatformCryptoResult Result = Decryptor->Reset(IV);

	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	Result = Decryptor->SetAuthTag(AuthTag);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	if (OutPlaintext.Num() < InCiphertext.Num())
	{
		// not enough space in plaintext output array
		return EPlatformCryptoResult::Failure;
	}

	int32 UpdateBytesWritten = 0;
	Result = Decryptor->Update(InCiphertext, OutPlaintext, UpdateBytesWritten);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	int32 FinalizeBytesWritten = 0;
	Result = Decryptor->Finalize(TArrayView<uint8>(OutPlaintext.GetData() + UpdateBytesWritten, OutPlaintext.Num() - UpdateBytesWritten), FinalizeBytesWritten);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	if (UpdateBytesWritten + FinalizeBytesWritten != InCiphertext.Num())
	{
		// AES GCM mode always decrypts to same amount of bytes as ciphertext
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FAESGCMHandlerComponent::Encrypt(TArrayView<uint8> OutCipherText, const TArrayView<const uint8> InPlaintext, const TArrayView<const uint8> IV, TArrayView<uint8> OutAuthTag)
{
	EPlatformCryptoResult Result = Encryptor->Reset(IV);

	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	if (OutCipherText.Num() < InPlaintext.Num())
	{
		// not enough space in ciphertext output array
		return EPlatformCryptoResult::Failure;
	}

	int32 UpdateBytesWritten = 0;
	Result = Encryptor->Update(InPlaintext, OutCipherText, UpdateBytesWritten);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	int32 FinalizeBytesWritten = 0;
	Result = Encryptor->Finalize(TArrayView<uint8>(OutCipherText.GetData() + UpdateBytesWritten, OutCipherText.Num() - UpdateBytesWritten), FinalizeBytesWritten);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	int32 AuthTagBytesWritten = 0;
	Result = Encryptor->GenerateAuthTag(OutAuthTag, AuthTagBytesWritten);
	if (Result != EPlatformCryptoResult::Success)
	{
		return Result;
	}

	if (UpdateBytesWritten + FinalizeBytesWritten != InPlaintext.Num())
	{
		// AES GCM mode always encrypts to same amount of bytes as plaintext
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

int32 FAESGCMHandlerComponent::GetReservedPacketBits() const
{
	// Worst case includes padding up to the next whole byte and the IV and AuthTag
	// For unencrypted packets it is padding plus an extra one byte, which is smaller
	return 7 + (IVSizeInBytes + AuthTagSizeInBytes) * 8;
}

void FAESGCMHandlerComponent::CountBytes(FArchive& Ar) const
{
	FEncryptionComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(FEncryptionComponent);
	Ar.CountBytes(SizeOfThis, SizeOfThis);

	/*
	Note, as of now, EncryptionContext is just typedef'd, but none of the base
	types actually allocated memory directly in their classes (although there may be
	global state).
	if (FEncryptionContext const * const LocalContext = EncrpytionContext.Get())
	{
		LocalContext->CountBytes(Ar);
	}
	*/
}

bool FAESGCMHandlerComponent::SupportsParallelConnectionOutgoing() const
{
	return true;
}
