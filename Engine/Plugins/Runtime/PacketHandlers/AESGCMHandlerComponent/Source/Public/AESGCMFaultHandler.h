// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Net/Core/Connection/NetResultManager.h"

#include "AESGCMFaultHandler.generated.h"


// Forward declarations
namespace UE
{
	namespace Net
	{
		class FNetConnectionFaultRecoveryBase;
	}
}

/**
 * AESGCM net error types, for NetConnection fault handling
 */
UENUM()
enum class EAESGCMNetResult : uint8
{
	Unknown,
	Success,

	/** AES GCM Packet missing Initialization Vector */
	AESMissingIV,

	/** AES GCM Packet missing Authentication Tag */
	AESMissingAuthTag,

	/** AES GCM Packet missing payload/ciphertext */
	AESMissingPayload,

	/** AES GCM Packet Decryption failed */
	AESDecryptionFailed,

	/** AES GCM Packet had zero last byte (no termination bit) */
	AESZeroLastByte
};

DECLARE_NETRESULT_ENUM(EAESGCMNetResult);

AESGCMHANDLERCOMPONENT_API const TCHAR* LexToString(EAESGCMNetResult Enum);



/**
 * AESGCM Fault Handler - implements fault handling for AESGCM net errors, tied to the main NetConnection fault recovery implementation
 */
class FAESGCMFaultHandler final : public UE::Net::FNetResultHandler
{
	friend class FAESGCMHandlerComponent;

private:
	void InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery);

	virtual UE::Net::EHandleNetResult HandleNetResult(UE::Net::FNetResult&& InResult) override;

private:
	/** The NetConnection FaultRecovery instance, which we forward fault counting to */
	UE::Net::FNetConnectionFaultRecoveryBase* FaultRecovery = nullptr;

	/** This fault handlers allocated counter index, within the NetConnection FaultRecovery instance */
	int32 CounterIndex = INDEX_NONE;
};
