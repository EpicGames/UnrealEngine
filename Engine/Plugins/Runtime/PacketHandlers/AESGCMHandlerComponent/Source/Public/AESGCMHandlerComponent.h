// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EncryptionComponent.h"
#include "AESGCMFaultHandler.h"
#include "IPlatformCrypto.h" // IWYU pragma: keep
#include "Containers/StaticArray.h"

struct FBitWriter;
struct FEncryptionData;
struct FOutPacketTraits;

/*
* AES256 GCM block encryption component.
*/
class FAESGCMHandlerComponent : public FEncryptionComponent
{
	friend class FAESGCMHandlerComponentTests;
public:
	/**
	 * Default constructor that leaves the Key empty, and encryption disabled.
	 * You must set the key before enabling encryption, or before receiving encrypted
	 * packets, or those operations will fail.
	 */
	AESGCMHANDLERCOMPONENT_API FAESGCMHandlerComponent();

	// This handler uses AES256, which has 32-byte keys.
	static const int32 KeySizeInBytes = 32;

	// This handler uses AES256, which has 32-byte keys.
	static const int32 BlockSizeInBytes = 16;

	static const int32 IVSizeInBytes = 12;
	static const int32 AuthTagSizeInBytes = 16;

	// Replace the key used for encryption with NewKey if NewKey is exactly KeySizeInBytes long.
	AESGCMHANDLERCOMPONENT_API virtual void SetEncryptionData(const FEncryptionData& EncryptionData) override;

	// After calling this, future outgoing packets will be encrypted (until a call to DisableEncryption).
	AESGCMHANDLERCOMPONENT_API virtual void EnableEncryption() override;

	// After calling this, future outgoing packets will not be encrypted (until a call to DisableEncryption).
	AESGCMHANDLERCOMPONENT_API virtual void DisableEncryption() override;

	// Returns true if encryption is currently enabled.
	AESGCMHANDLERCOMPONENT_API virtual bool IsEncryptionEnabled() const override;

	// HandlerComponent interface
	AESGCMHANDLERCOMPONENT_API virtual void Initialize() override;
	AESGCMHANDLERCOMPONENT_API virtual void InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery) override;
	AESGCMHANDLERCOMPONENT_API virtual bool IsValid() const override;
	AESGCMHANDLERCOMPONENT_API virtual void Incoming(FIncomingPacketRef PacketRef) override;
	AESGCMHANDLERCOMPONENT_API virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;
	AESGCMHANDLERCOMPONENT_API virtual int32 GetReservedPacketBits() const override;
	AESGCMHANDLERCOMPONENT_API virtual void CountBytes(FArchive& Ar) const override;

	// Whether or not this HandlerComponent supports running its Outgoing logic on multiple separate connections simultaneously.
	virtual bool SupportsParallelConnectionOutgoing() const override;

private:
	TUniquePtr<FEncryptionContext> EncryptionContext;

	TUniquePtr<IPlatformCryptoDecryptor> Decryptor;
	TUniquePtr<IPlatformCryptoEncryptor> Encryptor;

	// IV used for encryption
	TStaticArray<uint8, IVSizeInBytes> OutIV;

	bool bEncryptionEnabled;

	/** Fault handler for AESGCM-specific errors, that may trigger NetConnection Close */
	FAESGCMFaultHandler AESGCMFaultHandler;

	EPlatformCryptoResult Decrypt(TArrayView<uint8> OutPlainText, const TArrayView<const uint8> InCiphertext, const TArrayView<const uint8> IV, const TArrayView<const uint8> AuthTag);
	EPlatformCryptoResult Encrypt(TArrayView<uint8> OutCipherText, const TArrayView<const uint8> InPlaintext, const TArrayView<const uint8> IV, TArrayView<uint8> OutAuthTag);
};


/**
 * The public interface to this module.
 */
class FAESGCMHandlerComponentModule : public FPacketHandlerComponentModuleInterface
{
public:
	/* Creates an instance of this component */
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};
